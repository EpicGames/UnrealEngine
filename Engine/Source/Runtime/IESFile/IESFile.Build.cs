// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IESFile : ModuleRules
{
	public IESFile(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
	}
}
