// Copyright Epic Games, Inc. All Rights Reserved.


#include "IESLoader.h"

#include "IESConverter.h"
#include "Math/RandomStream.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, IESFile);

enum EIESVersion
{
	EIESV_1986,	// IES LM-63-1986
	EIESV_1991, // IES LM-63-1991
	EIESV_1995, // IES LM-63-1995
	EIESV_2002, // IES LM-63-2002
};

enum class EIESPhotometricType
{
	TypeC = 1,
	TypeB = 2,
	TypeA = 3
};

DEFINE_LOG_CATEGORY_STATIC(LogIESLoader, Log, All);


// space and return
static void JumpOverWhiteSpace(const uint8*& BufferPos)
{
	while(*BufferPos)
	{
		if(*BufferPos == 13 && *(BufferPos + 1) == 10)
		{
			BufferPos += 2;
			continue;
		}
		else if(*BufferPos <= ' ')
		{
			// Skip tab, space and invisible characters
			++BufferPos;
			continue;
		}

		break;
	}
}

static void GetLineContent(const uint8*& BufferPos, char Line[256], bool bStopOnWhitespace)
{
	JumpOverWhiteSpace(BufferPos);

	char* LinePtr = Line;

	uint32 i;

	for(i = 0; i < 255; ++i)
	{
		if(*BufferPos == 0)
		{
			break;
		}
		else if(*BufferPos == '\r' && *(BufferPos + 1) == '\n')
		{
			BufferPos += 2;
			break;
		}
		else if(*BufferPos == '\n')
		{
			++BufferPos;
			break;
		}
		else if(bStopOnWhitespace && (*BufferPos <= ' '))
		{
			// tab, space, invisible characters
			++BufferPos;
			break;
		}

		*LinePtr++ = *BufferPos++;
	}

	Line[i] = 0;
}


// @return success
static bool GetFloat(const uint8*& BufferPos, float& ret)
{
	char Line[256];

	GetLineContent(BufferPos, Line, true);

	ret = FCStringAnsi::Atof(Line);
	return true;
}


static bool GetInt(const uint8*& BufferPos, int32& ret)
{
	char Line[256];

	GetLineContent(BufferPos, Line, true);

	ret = FCStringAnsi::Atoi(Line);
	return true;
}



FIESLoader::FIESLoader(const uint8* Buffer, uint32 BufferLength)
	: Brightness(0)
	, CachedIntegral(MAX_FLT)
	, PhotometricType(EIESPhotometricType::TypeC)
	, Error(TEXT("No data loaded"))
{
	check(!IsValid());

	// Make sure BufferLength plus the terminator doesn't overflow the size type used by TArray (which is signed).
	BufferLength = FMath::Min<uint32>(BufferLength, MAX_int32 - 1);

	TArray<uint8> ASCIIFile;

	// add 0 termination for easier parsing
	{
		ASCIIFile.AddUninitialized(BufferLength + 1);
		FMemory::Memcpy(ASCIIFile.GetData(), Buffer, BufferLength);
		ASCIIFile[BufferLength] = 0;
	}

	Load(ASCIIFile.GetData());
}


#define PARSE_FLOAT(x) float x; if(!GetFloat(BufferPos, x)) return
#define PARSE_INT(x) int32 x; if(!GetInt(BufferPos, x)) return

void FIESLoader::Load(const uint8* Buffer)
{
	// file format as described here: http://www.ltblight.com/English.lproj/LTBLhelp/pages/iesformat.html

	const uint8* BufferPos = Buffer;

	EIESVersion Version = EIESV_1986;
	{
		Error = TEXT("VersionError");

		char Line1[256];

		GetLineContent(BufferPos, Line1, false);

		if(FCStringAnsi::Stricmp(Line1, "IESNA:LM-63-1995") == 0)
		{
			Version = EIESV_1995;
		}
		else if(FCStringAnsi::Stricmp(Line1, "IESNA91") == 0)
		{
			Version = EIESV_1991;
		}
		else if(FCStringAnsi::Stricmp(Line1, "IESNA:LM-63-2002") == 0)
		{
			Version = EIESV_2002;
		}
		else
		{
			Version = EIESV_1986;

			// Return buffer to start of file since line read was not the version
			BufferPos = Buffer;
		}
	}

	Error = TEXT("HeaderError");

	while(*BufferPos)
	{
		char Line[256];

		GetLineContent(BufferPos, Line, false);

		if(FCStringAnsi::Strcmp(Line, "TILT=NONE") == 0)
		{
			// at the moment we don't support only profiles with TILT=NONE
			break;
		}
		else if(FCStringAnsi::Strncmp(Line, "TILT=", 5) == 0)
		{
			// "TILT=NONE", "TILT=INCLUDE", and "TILT={filename}"
			// not supported yet, seems rare
			return;
		}
	}

	Error = TEXT("HeaderParameterError");

	PARSE_INT(LightCreationCount);

	if(LightCreationCount < 1)
	{
		Error = TEXT("Light count needs to be positive.");
		return;
	}

	// if there is any file with that - do we need to parse it differently?
	check(LightCreationCount >= 1);

	PARSE_FLOAT(LumensPerLamp);

	PARSE_FLOAT(CandelaMult);

	if(CandelaMult < 0)
	{
		Error = TEXT("CandelaMult is negative");
		return;
	}

	PARSE_INT(VAnglesNum);
	PARSE_INT(HAnglesNum);

	if(VAnglesNum < 0)
	{
		Error = TEXT("VAnglesNum is not valid");
		return;
	}

	if(HAnglesNum < 0)
	{
		Error = TEXT("HAnglesNum is not valid");
		return;
	}

	PARSE_INT(PhotometricTypeTemp);

	if ( PhotometricTypeTemp >= (int32)EIESPhotometricType::TypeC && PhotometricTypeTemp <= (int32)EIESPhotometricType::TypeA )
	{
		PhotometricType = EIESPhotometricType(PhotometricTypeTemp);
	}

	// 1:feet, 2:meter
	PARSE_INT(UnitType);

	PARSE_FLOAT(Width);
	PARSE_FLOAT(Length);
	PARSE_FLOAT(Height);

	PARSE_FLOAT(BallastFactor);
	PARSE_FLOAT(FutureUse);

//	check(FutureUse == 1.0f);

	PARSE_FLOAT(InputWatts);

	Error = TEXT("ContentError");

	{
		float MinSoFar = -FLT_MAX;

		VAngles.Empty(VAnglesNum);
		for(uint32 y = 0; y < (uint32)VAnglesNum; ++y)
		{
			PARSE_FLOAT(Value);

			if(Value < MinSoFar)
			{
				// binary search later relies on that
				Error = TEXT("V Values are not in increasing order");
				return;
			}

			MinSoFar = Value;
			VAngles.Add(Value);
		}
	}

	{
		float MinSoFar = -FLT_MAX;

		HAngles.Empty(HAnglesNum);
		for(uint32 x = 0; x < (uint32)HAnglesNum; ++x)
		{
			PARSE_FLOAT(Value);

			if(Value < MinSoFar)
			{
				// binary search later relies on that
				Error = TEXT("H Values are not in increasing order");
				return;
			}

			MinSoFar = Value;
			HAngles.Add(Value);
		}
	}

	CandelaValues.Empty(HAnglesNum * VAnglesNum);
	for(uint32 y = 0; y < (uint32)HAnglesNum; ++y)
	{
		for(uint32 x = 0; x < (uint32)VAnglesNum; ++x)
		{
			PARSE_FLOAT(Value);

			CandelaValues.Add(Value * CandelaMult);
		}
	}

	Error = TEXT("Unexpected content after candela values.");

	JumpOverWhiteSpace(BufferPos);

	if(*BufferPos)
	{
		// some files are terminated with "END"
		char Line[256];

		GetLineContent(BufferPos, Line, true);

		if(FCStringAnsi::Strcmp(Line, "END") == 0)
		{
			JumpOverWhiteSpace(BufferPos);
		}
	}

	if(*BufferPos)
	{
		Error = TEXT("Unexpected content after END.");
		return;
	}

	Error = nullptr;

	Brightness = ComputeMax(); // Use max candela as the brightness

	if(Brightness <= 0)
	{
		// some samples have -1, then the brightness comes from the samples
//		Brightness = ComputeFullIntegral();

		// use some reasonable value
		Brightness = 1000;
	}
}
#undef PARSE_FLOAT
#undef PARSE_INT

uint32 FIESLoader::GetWidth() const
{
	return 256;
}

uint32 FIESLoader::GetHeight() const
{
	return 256;
}

bool FIESLoader::IsValid() const
{
	return Error.IsEmpty();
}

float FIESLoader::GetBrightness() const
{
	return Brightness;
}

float FIESLoader::ExtractInRGBA16F(TArray<uint8>& OutData)
{
	check(IsValid());

	uint32 Width = GetWidth();
	uint32 Height = GetHeight();

	check(!OutData.Num());
	OutData.AddZeroed(Width * Height * sizeof(FFloat16Color));

	FFloat16Color* Out = (FFloat16Color*)OutData.GetData();

	float InvWidth = 1.0f / Width;
	float InvHeight = 1.0f / Height;
	float MaxValue = ComputeMax();
	float InvMaxValue= 1.0f / MaxValue;

	for(uint32 y = 0; y < Height; ++y)
	{
		float HFraction = y * InvHeight;

		for(uint32 x = 0; x < Width; ++x)
		{
			// 0..1
			float VFraction = x * InvWidth;

			// distort for better quality?
//				Fraction = Square(Fraction);

			float FloatValue = InvMaxValue * Interpolate2D(HFraction * 360.0f, VFraction * 180.0f);
			{
				FFloat16 HalfValue(FloatValue);

				FFloat16Color HalfColor;

				HalfColor.R = HalfValue;
				HalfColor.G = HalfValue;
				HalfColor.B = HalfValue;
				HalfColor.A = HalfValue;

				*Out++ = HalfColor;
			}
		}
	}

	return 1.f;
}

float FIESLoader::ComputeFullIntegral()
{
	// compute only if needed
	if(CachedIntegral == MAX_FLT)
	{
		// monte carlo integration
		// if quality is a problem we can improve on this algorithm or increase SampleCount

		// larger number costs more time but improves quality
		uint32 SampleCount = 1000000;

		FRandomStream RandomStream(0x1234);

		double Sum = 0;
		for(uint32 i = 0; i < SampleCount; ++i)
		{
			FVector Dir = RandomStream.GetUnitVector();

			// http://en.wikipedia.org/wiki/Spherical_coordinate_system

			// 0..180
			float VAngle = FMath::Acos(Dir.Z) / PI * 180;
			// 0..360
			float HAngle = FMath::Atan2(Dir.Y, Dir.X) / PI * 180 + 180;

			check(VAngle >= 0 && VAngle <= 180);
			check(HAngle >= 0 && HAngle <= 360);

			Sum += Interpolate2D(HAngle, VAngle);
		}

		CachedIntegral = Sum / SampleCount;
	}

	return CachedIntegral;
}



float FIESLoader::ComputeMax() const
{
	float ret = 0.0f;

	for(uint32 i = 0; i < (uint32)CandelaValues.Num(); ++i)
	{
		float Value = CandelaValues[i];

		ret = FMath::Max(ret, Value);
	}

	return ret;
}


float FIESLoader::ComputeFilterPos(float Value, const TArray<float>& SortedValues)
{
	check(SortedValues.Num());

	uint32 StartPos = 0;
	uint32 EndPos = SortedValues.Num() - 1;

	if(Value < SortedValues[StartPos])
	{
		return 0.0f;
	}

	if(Value > SortedValues[EndPos])
	{
		return (float)EndPos;
	}

	// binary search
	while(StartPos < EndPos)
	{
		uint32 TestPos = (StartPos + EndPos + 1) / 2;

		float TestValue = SortedValues[TestPos];

		if(Value >= TestValue)
		{
			// prevent endless loop
			check(StartPos != TestPos);

			StartPos = TestPos;
		}
		else
		{
			// prevent endless loop
			check(EndPos != TestPos - 1);

			EndPos = TestPos - 1;
		}
	}

	float LeftValue = SortedValues[StartPos];

	float Fraction = 0.0f;

	if(StartPos + 1 < (uint32)SortedValues.Num())
	{
		// if not at right border
		float RightValue = SortedValues[StartPos + 1];
		float DeltaValue = RightValue - LeftValue;

		if(DeltaValue > 0.0001f)
		{
			Fraction = (Value - LeftValue) / DeltaValue;
		}
	}

	return StartPos + Fraction;
}


float FIESLoader::InterpolatePoint(int X, int Y) const
{
	uint32 HAnglesNum = HAngles.Num();
	uint32 VAnglesNum = VAngles.Num();

	check(X >= 0);
	check(Y >= 0);

	X %= HAnglesNum;
	Y %= VAnglesNum;

	check(X < (int)HAnglesNum);
	check(Y < (int)VAnglesNum);

	return CandelaValues[Y + VAnglesNum * X];
}


float FIESLoader::InterpolateBilinear(float fX, float fY) const
{
	int X = (int)fX;
	int Y = (int)fY;

	float fracX = fX - X;
	float fracY = fY - Y;

	float p00 = InterpolatePoint(X + 0, Y + 0);
	float p10 = InterpolatePoint(X + 1, Y + 0);
	float p01 = InterpolatePoint(X + 0, Y + 1);
	float p11 = InterpolatePoint(X + 1, Y + 1);
	
	float p0 = FMath::Lerp(p00, p01, fracY); 
	float p1 = FMath::Lerp(p10, p11, fracY); 

	return FMath::Lerp(p0, p1, fracX);
}

float FIESLoader::Interpolate2D(float HAngle, float VAngle) const
{
	float u = 0.0f;
	float v = 0.0f;

	if (PhotometricType == EIESPhotometricType::TypeA)
	{
		float X = FMath::Cos(FMath::DegreesToRadians(VAngle));
		float Y = FMath::Sin(FMath::DegreesToRadians(VAngle)) * FMath::Sin(FMath::DegreesToRadians(HAngle));
		float Z = FMath::Sin(FMath::DegreesToRadians(VAngle)) * FMath::Cos(FMath::DegreesToRadians(HAngle));


		float NewVAngle = FMath::RadiansToDegrees(FMath::Asin(Z));
		float NewHAngle = -FMath::RadiansToDegrees(FMath::Atan2(Y, X));

		u = ComputeFilterPos(NewHAngle, HAngles);
		v = ComputeFilterPos(NewVAngle, VAngles);
	}
	else
	{
		// Support symmetry, per the IES format doc:
		// 0     There is only one horizontal angle, implying that the luminaire is laterally symmetric in all photometric planes.
		// 90    The luminaire is assumed to be symmetric in each quadrant.
		// 180   The luminaire is assumed to be bilaterally symmetric about the 0-180 degree photometric plane.
		// 360   The luminaire is assumed to exhibit no lateral symmetry.
		//
		// A luminaire that is bilaterally symmetric about the 90-270 degree
		// photometric plane will have a first value of 90 degrees and a last value of 270 degrees.

		if (HAngles.Num() > 0)
		{
			if (HAngles.Last() > 0.f && HAngle > HAngles.Last())
			{
				bool bFlipHAngle = false;

				int32 HQuadrant = (int32)HAngle / HAngles.Last();

				if ( HQuadrant == 1 || HQuadrant == 3 )
				{
					bFlipHAngle = true;
				}

				if ( bFlipHAngle )
				{
					HAngle = HAngles.Last() - FMath::Fmod(HAngle, HAngles.Last());
				}
				else
				{
					HAngle = FMath::Fmod(HAngle, HAngles.Last());
				}
			}
			else if (HAngles[0] > 0.f && HAngle < HAngles[0])
			{
				HAngle = HAngles.Last() - HAngles[0] - HAngle;
			}
		}

		u = ComputeFilterPos(HAngle, HAngles);
		v = ComputeFilterPos(VAngle, VAngles);
	}
	

	return InterpolateBilinear(u, v);
}

float FIESLoader::Interpolate1D(float VAngle) const
{
	float v = ComputeFilterPos(VAngle, VAngles);

	float ret = 0.0f;

	uint32 HAnglesNum = (uint32)HAngles.Num();

	for(uint32 i = 0; i < HAnglesNum; ++i)
	{
		ret += InterpolateBilinear((float)i, v);
	}

	return ret / HAnglesNum;
}

FIESConverter::FIESConverter(const uint8* Buffer, uint32 BufferLength)
{
	Impl = MakeShared<FIESLoader>(Buffer, BufferLength);
	if (Impl->IsValid())
	{
		Multiplier = Impl->ExtractInRGBA16F(RawData);
	}
}

bool FIESConverter::IsValid() const
{
	return Impl->IsValid();
}

const TCHAR* FIESConverter::GetError() const
{
	return Impl->GetError();
}

uint32 FIESConverter::GetWidth() const
{
	return Impl->GetWidth();
}

uint32 FIESConverter::GetHeight() const
{
	return Impl->GetHeight();
}

float FIESConverter::GetBrightness() const
{
	return Impl->GetBrightness();
}
