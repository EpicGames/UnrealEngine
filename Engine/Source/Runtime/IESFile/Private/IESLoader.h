// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

//class Error;

enum class EIESPhotometricType;

/**
 * To load the IES file image format. IES files exist for many real world lights. The file stores how much light is emitted in a specific direction.
 * The data is usually measured but tools to paint IES files exist.
 */
class FIESLoader
{
public:
	/** is loading the file, can take some time, success can be checked with IsValid() */
	FIESLoader(const uint8* Buffer, uint32 BufferLength);

	// @return Multiplier as the texture is normalized
	float ExtractInRGBA16F(TArray<uint8>& OutData);

	bool IsValid() const;

	uint32 GetWidth() const;
	uint32 GetHeight() const;

	/** @return in Lumens */
	float GetBrightness() const;

	const TCHAR* GetError() const
	{
		return Error.IsEmpty() ? nullptr : *Error;
	}

private:

	TArray<float> HAngles;
	TArray<float> VAngles;
	TArray<float> CandelaValues;

	/** in Lumens, always >0 */
	float Brightness;

	/** used by ComputeFullIntegral(), integrated over the unit sphere, to avoid computing it twice, -1 if not computed yet */
	float CachedIntegral;

	/** The photometric type associated with the IES file being loaded */
	EIESPhotometricType PhotometricType;

	// will set Error to 0 on success
	void Load(const uint8* Buffer);

	// 0 if no error, otherwise points to an ASCII error string
	FString Error;

	float ComputeMax() const;

	// integrate over the unit sphere
	// @return in Candela
	float ComputeFullIntegral();

	static float ComputeFilterPos(float Value, const TArray<float>& SortedValues);

	// low level code, only used by InterpolateBilinear
	// @param X 0..HAngles.size()-1
	// @param Y 0..VAngles.size()-1
	float InterpolatePoint(int X, int Y) const;

	// low level code, used by Interpolate2D() and Interpolate1D()
	// @param fX 0..HAngles.size()-1
	// @param fY 0..VAngles.size()-1
	float InterpolateBilinear(float fX, float fY) const;

	// high level code to compute the Candela value for a given direction
	// @param HAngle n degrees e.g. 0..360
	// @param VAngle n degrees e.g. 0..180
	float Interpolate2D(float HAngle, float VAngle) const;

	// like Interpolate2D but also integrates over HAngle
	// @param VAngle n degrees e.g. 0..180
	float Interpolate1D(float VAngle) const;
};
