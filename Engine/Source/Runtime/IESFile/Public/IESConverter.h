// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * To load the IES file image format. IES files exist for many real world lights. The file stores how much light is emitted in a specific direction.
 * The data is usually measured but tools to paint IES files exist.
 */
class FIESConverter
{
public:
	/** is loading the file, can take some time, success can be checked with IsValid() */
	IESFILE_API FIESConverter(const uint8* Buffer, uint32 BufferLength);

	/**
	 * @return true if the photometric data are valid
	 * @note Call GetError to get a description of the error
	 */
	IESFILE_API bool IsValid() const;

	/**
	 * @return a brief description of the reason why the IES data is invalid
	 */
	IESFILE_API const TCHAR* GetError() const;

	/**
	 * @return Multiplier as the texture is normalized
	 */
	float GetMultiplier() const
	{
		return Multiplier;
	}

	/**
	 * @return data to create UTextureProfile
	 */
	const TArray<uint8>& GetRawData() const
	{
		return RawData;
	}

	IESFILE_API uint32 GetWidth() const;
	IESFILE_API uint32 GetHeight() const;

	/**
	 * @return brightness in Lumens
	 */
	IESFILE_API float GetBrightness() const;

private:
	TArray<uint8> RawData;
	float Multiplier;

	TSharedPtr<class FIESLoader> Impl;
};
